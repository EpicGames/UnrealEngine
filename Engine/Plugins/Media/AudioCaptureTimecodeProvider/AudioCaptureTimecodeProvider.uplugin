{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Audio Capture Timecode Provider",
	"Description" : "Decodes an LTC signal (linear timcode) from a live audio capture device (ie. the computer audio jack).",
	"Category" : "Media Players",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name": "AudioCaptureTimecodeProvider",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins":
	[
		{
			"Name": "AudioCapture",
			"Enabled": true,
			"PlatformAllowList": [ "Win64" ]
		},
		{
			"Name": "LinearTimecode",
			"Enabled": true,
			"PlatformAllowList": [ "Win64" ]
		}
	]
}