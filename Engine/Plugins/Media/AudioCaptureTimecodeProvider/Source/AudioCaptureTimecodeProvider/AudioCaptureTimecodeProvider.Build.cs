// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AudioCaptureTimecodeProvider : ModuleRules
	{
		public AudioCaptureTimecodeProvider(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AudioCapture",
                    "AudioCaptureCore",
                    "Core",
					"CoreUObject",
					"Engine",
					"LinearTimecode",
					"TimeManagement"
				}
			);
		}
	}
}
