// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureTimecodeProvider.h"
#include "AudioCaptureTimecodeProviderModule.h"

#include "AudioCapture.h"
#include "HAL/CriticalSection.h"
#include "Misc/CommandLine.h"
#include "LinearTimecodeDecoder.h"
#include "Stats/StatsMisc.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/* FLinearTimecodeAudioCaptureCustomTimeStepImplementation implementation
*****************************************************************************/
struct UAudioCaptureTimecodeProvider::FLinearTimecodeAudioCaptureCustomTimeStepImplementation
{
public:
	FLinearTimecodeAudioCaptureCustomTimeStepImplementation(UAudioCaptureTimecodeProvider* InOwner)
		: bWarnedAboutTheInvalidAudioChannel(false)
		, bFrameRateReach0Counter(0)
		, Owner(InOwner)
		, bStopRequested(false)
	{
	}

	~FLinearTimecodeAudioCaptureCustomTimeStepImplementation()
	{
		bStopRequested = true;
		AudioCapture.StopStream(); // will make sure OnAudioCapture() is completed
		AudioCapture.CloseStream();
	}

	bool Init()
	{
		// OnAudioCapture is called when the buffer is full.
		//We want a fast timecode detection but we don't want to be called too often.
		const int32 NumberCaptureFrames = 64;

		Audio::FOnAudioCaptureFunction OnCapture = [this](const void* AudioData, int32 NumFrames, int32 NumChannels, int32 SampleRate, double StreamTime, bool bOverFlow)
		{
			OnAudioCapture((const float*)AudioData, NumFrames, NumChannels, StreamTime, bOverFlow);
		};

		Audio::FAudioCaptureDeviceParams Params = Audio::FAudioCaptureDeviceParams();

		if (!AudioCapture.OpenAudioCaptureStream(Params, MoveTemp(OnCapture), NumberCaptureFrames))
		{
			UE_LOGF(LogAudioCaptureTimecodeProvider, Error, "Can't open the default capture stream for %ls.", *Owner->GetName());
			return false;
		}

		// set limits to help decoder
		{
			constexpr int32 MinFps = 20;
			constexpr int32 MaxFps = 34;
			constexpr int32 NumLtcBits = 80;
			constexpr int32 NumLtcHalfBits = NumLtcBits * 2;
			TimecodeDecoder.MinSamplesPerEdge = AudioCapture.GetSampleRate() / (MaxFps * NumLtcHalfBits);
			TimecodeDecoder.MaxSamplesPerEdge = AudioCapture.GetSampleRate() / (MinFps * NumLtcBits);
		}

		check(AudioCapture.IsStreamOpen());
		check(!AudioCapture.IsCapturing());

		if (!AudioCapture.StartStream())
		{
			AudioCapture.CloseStream();
			UE_LOGF(LogAudioCaptureTimecodeProvider, Error, "Can't start the default capture stream for %ls.", *Owner->GetName());
			return false;
		}

		return true;
	}

	void OnAudioCapture(const float* AudioData, int32 NumFrames, int32 NumChannels, double StreamTime, bool bOverflow)
	{
		check(Owner);

		if (bStopRequested)
		{
			return;
		}
		
		int32 AudioChannelIndex = FMath::Clamp(Owner->AudioChannel-1, 0, NumChannels-1);
		if (!bWarnedAboutTheInvalidAudioChannel && AudioChannelIndex != Owner->AudioChannel-1)
		{
			bWarnedAboutTheInvalidAudioChannel = true;
			UE_LOGF(LogAudioCaptureTimecodeProvider, Warning, "The AudioChannel provided is invalid for %ls. The number of channels available is %d.", *Owner->GetName(), NumChannels);

		}

		AudioData += AudioChannelIndex;

		int32 NumSamples = NumChannels * NumFrames;
		const float* End = AudioData + NumSamples;

		for (const float* Begin = AudioData; Begin != End; Begin += NumChannels)
		{
			if (TimecodeDecoder.Sample(*Begin, CurrentDecodingTimecode))
			{
				if (bStopRequested)
				{
					return;
				}

				{
					FScopeLock Lock(&CriticalSection);
					Timecode = CurrentDecodingTimecode;
				}

				if (Owner->bDetectFrameRate)
				{
					if (Timecode.Timecode.Frames == 0)
					{
						++bFrameRateReach0Counter;
						if (bFrameRateReach0Counter > 1) // Did we loop enough frame to know the frame rates. Assume non drop frame.
						{
							Owner->SynchronizationState = ETimecodeProviderSynchronizationState::Synchronized;
						}
					}
				}
				else
				{
					Owner->SynchronizationState = ETimecodeProviderSynchronizationState::Synchronized;
				}
			}
		}
	}

public:
	/** Audio capture object */
	Audio::FAudioCapture AudioCapture;

	/** Current time code decoded by the TimecodeDecoder */
	FDropTimecode CurrentDecodingTimecode;

	/** LTC decoder */
	FLinearTimecodeDecoder TimecodeDecoder;

	/** Lock to access the Timecode */
	FCriticalSection CriticalSection;

	/** Warn about the invalid audio channel the user requested */
	bool bWarnedAboutTheInvalidAudioChannel;

	/** Know when we have done synchronizing the FrameRate */
	int32 bFrameRateReach0Counter;

	/** Current time code decoded by the TimecodeDecoder */
	FDropTimecode Timecode;

	/** Owner of the implementation */
	UAudioCaptureTimecodeProvider* Owner;

	/** If the Owner requested the implementation to stop processing */
	volatile bool bStopRequested;
};

/* UAudioCaptureTimecodeProvider
*****************************************************************************/
UAudioCaptureTimecodeProvider::UAudioCaptureTimecodeProvider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, AudioChannel(1)
	, Implementation(nullptr)
	, SynchronizationState(ETimecodeProviderSynchronizationState::Closed)
{
}

/* UTimecodeProvider interface implementation
*****************************************************************************/
bool UAudioCaptureTimecodeProvider::FetchTimecode(FQualifiedFrameTime& OutFrameTime)
{
	OutFrameTime = FQualifiedFrameTime(GetTimecodeInternal(), GetFrameRateInternal());
	return true;
}

FTimecode UAudioCaptureTimecodeProvider::GetTimecodeInternal() const
{
	FTimecode Result;
	{
		if (Implementation)
		{
			FScopeLock Lock(&Implementation->CriticalSection);
			Result = Implementation->Timecode.Timecode;
		}
	}

	if (bDetectFrameRate)
	{
		Result.bDropFrameFormat = bAssumeDropFrameFormat;
	}
	else
	{
		Result.bDropFrameFormat = FTimecode::IsDropFormatTimecodeSupported(GetFrameRateInternal());
	}

	return Result;
}

FFrameRate UAudioCaptureTimecodeProvider::GetFrameRateInternal() const
{
	FFrameRate Result = FrameRate;
	if (bDetectFrameRate)
	{
		int32 DetectedFrameRate = 30;
		if (Implementation)
		{
			FScopeLock Lock(&Implementation->CriticalSection);
			DetectedFrameRate = Implementation->Timecode.FrameRate;
		}

		if (bAssumeDropFrameFormat)
		{
			if (DetectedFrameRate == 24 || DetectedFrameRate == 23)
			{
				Result = FFrameRate(24000, 1001);
			}
			else if (DetectedFrameRate == 30 || DetectedFrameRate == 29)
			{
				Result = FFrameRate(30000, 1001);
			}
			else if (DetectedFrameRate == 60 || DetectedFrameRate == 59)
			{
				Result = FFrameRate(60000, 1001);
			}
			else
			{
				Result = FFrameRate(DetectedFrameRate, 1);
			}
		}
		else
		{
			Result = FFrameRate(DetectedFrameRate, 1);
		}
	}
	return Result;
}

bool UAudioCaptureTimecodeProvider::Initialize(class UEngine* InEngine)
{
	if (IsRunningCommandlet() && !FParse::Param(FCommandLine::Get(), TEXT("useaudiocapturetimecode")))
	{
		UE_LOGF(LogAudioCaptureTimecodeProvider, Display, "Audio Capture Timecode Provided initilization was skipped because UE is running in a commandlet. Use -useaudiocapturetimecode to force the initialization.");		
		return false;
	}
	
	if (!ensure(Implementation == nullptr))
	{
		delete Implementation;
	}

	Implementation = new FLinearTimecodeAudioCaptureCustomTimeStepImplementation(this);
	bool bInitialized = Implementation->Init();
	if (!bInitialized)
	{
		SynchronizationState = ETimecodeProviderSynchronizationState::Error;
		delete Implementation;
		Implementation = nullptr;
	}
	SynchronizationState = ETimecodeProviderSynchronizationState::Synchronizing;
	return bInitialized;
}

void UAudioCaptureTimecodeProvider::Shutdown(class UEngine* InEngine)
{
	if (Implementation)
	{
		SynchronizationState = ETimecodeProviderSynchronizationState::Closed;
		delete Implementation;
		Implementation = nullptr;
	}
}

void UAudioCaptureTimecodeProvider::BeginDestroy()
{
	delete Implementation;
	Super::BeginDestroy();
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS