// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureTimecodeProviderModule.h"

#include "AudioCaptureTimecodeProvider.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAudioCaptureTimecodeProvider);

IMPLEMENT_MODULE(FDefaultModuleImpl, AudioCaptureTimecodeProvider)
