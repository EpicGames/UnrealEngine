// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenlockedTimecodeProvider.h"

#include "Misc/FrameRate.h"
#include "DropTimecode.h"

#include "AudioCaptureTimecodeProvider.generated.h"

/**
 * Read the LTC from the audio capture device.
 */
UCLASS(Blueprintable, editinlinenew)
class AUDIOCAPTURETIMECODEPROVIDER_API UAudioCaptureTimecodeProvider : public UGenlockedTimecodeProvider
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Detect the frame rate from the audio source.
	 * It may takes some extra time before the frame rate is properly detected.
	 */
	UPROPERTY(EditAnywhere, Category = TimecodeProvider)
	bool bDetectFrameRate;

	/** When detecting the frame rate. Assume the frame rate is a drop frame format. */
	UPROPERTY(EditAnywhere, Category = TimecodeProvider, meta = (EditCondition = "bDetectFrameRate"))
	bool bAssumeDropFrameFormat;

	/**
	 * Frame expected from the audio source.
	 */
	UPROPERTY(EditAnywhere, Category = TimecodeProvider, meta = (EditCondition = "!bDetectFrameRate"))
	FFrameRate FrameRate;

	/**
	 * Index of the Channel to used for the capture.
	 */
	UPROPERTY(EditAnywhere, Category = TimecodeProvider, meta = (ClampMin = 1))
	int32 AudioChannel;

public:
	//~ UTimecodeProvider interface
	virtual bool FetchTimecode(FQualifiedFrameTime& OutFrameTime) override;
	virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const override { return SynchronizationState; }
	virtual bool Initialize(class UEngine* InEngine) override;
	virtual void Shutdown(class UEngine* InEngine) override;

	//~ UObject interface
	virtual void BeginDestroy() override;
	
private:
	FTimecode GetTimecodeInternal() const;
	FFrameRate GetFrameRateInternal() const;

	/** Audio capture object dealing with getting audio callbacks */
	struct FLinearTimecodeAudioCaptureCustomTimeStepImplementation;
	friend FLinearTimecodeAudioCaptureCustomTimeStepImplementation;
	FLinearTimecodeAudioCaptureCustomTimeStepImplementation* Implementation;

	/** The current SynchronizationState of the TimecodeProvider*/
	ETimecodeProviderSynchronizationState SynchronizationState;
};
